// Copyright Epic Games, Inc. All Rights Reserved.

#include "CSVProfilerUtils.h"

#include "Async/ParallelFor.h"
#include "HAL/FileManager.h"
#include "Misc/Compression.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "Misc/ScopedSlowTask.h"
#include "String/LexFromString.h"
#include "String/ParseTokens.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"

#define LOCTEXT_NAMESPACE "CsvUtils"

namespace CsvUtils
{

namespace CsvUtils::Private
{

static const FText IncorrectFormatText(
	LOCTEXT("IncorrectFormat", "Incorrect file format - couldn't read expected magic."));
static const FText UnsupportedVersionText(LOCTEXT("UnsupportedVersion", "File is of an unsupported version."));
static const FText UnsupportedCompressionTypeText(
	LOCTEXT("UnsupportedCompressionType", "File uses an unsupported compression type."));
static const FText UncompressedFormatSupportText(
	LOCTEXT("UncompressedFormatSupport", "Uncompressed format loading is not yet supported."));
static const FText SampleDataNotFoundText(
	LOCTEXT("SampleDataNotFound", "Unable to find sample with name {SampleName} to serialize."));

static constexpr FStringView SampleNameKey(TEXTVIEW("SampleName"));

// Should be kept up-to-date with CSVStats.CsvBinVersion
enum class ECsvBinVersion : int32
{
	PreRelease = 1,
	InitialRelease,
	CompressionSupportAndFlags,

	COUNT,
	CURRENT = COUNT - 1
};

// Should be kept up-to-date with CSVStats.CsvBinFlags
enum class ECsvBinFlags : uint32
{
	None = 0,
	HasMetadata = 0x00000001,
};
ENUM_CLASS_FLAGS(ECsvBinFlags);

// Should be kept up-to-date with CSVStats.CsvBinCompressionType
enum class CsvBinCompressionType : uint8
{
	MsDeflate
};

// Should be kept up-to-date with CSVStats.ECsvBinCompressionLevel
enum class ECsvBinCompressionLevel : uint8
{
	None,
	Min,
	Max
};

static bool LineIsMetadata(FStringView Line)
{
	if (Line.TrimStartAndEnd().StartsWith(TEXT('[')))
	{
		return true;
	}
	return false;
}

/**
* Helper function to serialize metadata from the parameter line.
* @param Line Text line to read the metadata from.
* @param OutMetadata The container to store the read metadata in.
*/
void SerializeMetadataText(const FString& Line, TMap<FString, FString>& OutMetadata)
{
	// Initialize state tracking variables
	bool bIsKey = false;
	FString CurrentKey;

	// Split the metadata line into segments at commas
	const TCHAR* LinePtr = *Line;
	FString Token;
	while (FParse::Token(LinePtr, Token, false, ','))
	{		
		// Check if this is a key (enclosed in square brackets)
		if (!bIsKey && Token.StartsWith(TEXT("[")) && Token.EndsWith(TEXT("]")))
		{
			// Extract key without brackets and convert to lowercase
			CurrentKey = MoveTemp(Token);
			CurrentKey.MidInline(1, CurrentKey.Len() - 2);
			CurrentKey.ToLowerInline();
			bIsKey = true;
			continue;
		}

		// Handle the value
		if (bIsKey)
		{
			// Add or append to existing value
			if (FString* ExistingValue = OutMetadata.Find(CurrentKey))
			{
				*ExistingValue += TEXT(",") + Token;
			}
			else
			{
				OutMetadata.Add(MoveTemp(CurrentKey), MoveTemp(Token));
			}

			bIsKey = false;
		}
	}
}

/**
 * Serializes binary CSV profiler data. Based on the C# code found in CsvStats.ReadCSVFromLines.
 * @param Lines Text lines to serialize from.
 * @param OutCapture Output container to store the read capture in.
 * @param OutErrors Optional output array to hold any reported errors.
 * @return True if serialization of the capture succeeded.
 */
bool SerializeText(TConstArrayView<FString> Lines, FCsvProfilerCapture& OutCapture, TArray<FText>* OutErrors = nullptr)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(Private::SerializeText);

	if (Lines.IsEmpty())
	{
		return false;
	}

	// Use the first line as the header view, unless the metadata tells us to use the row at the end instead.
	const FString* HeaderRow = &Lines[0];

	// Remove the header row from the view.
	Lines.RightChopInline(1);

	TMap<FString, FString> Metadata;
	if (LineIsMetadata(Lines.Last()))
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(Private::SerializeText::Metadata);

		// Serialize the metadata.
		SerializeMetadataText(Lines.Last(), Metadata);

		// Remove the metadata line.
		Lines.LeftChopInline(1);

		// New CSVs from the csv profiler have a header row at the end of the file,
		// since the profiler writes out the file incrementally.
		const FString* HasHeaderRowAtEndValue = Metadata.Find("hasheaderrowatend");
		if (HasHeaderRowAtEndValue && *HasHeaderRowAtEndValue == "1")
		{
			// Swap the header row for the one at the end of the file.
			HeaderRow = &Lines[Lines.Num() - 1];

			// Remove the end header row.
			Lines.LeftChopInline(1);
		}
	}

	// We should be left with only sample lines.
	const int32 NumSamples = Lines.Num();

	TArray<FString> SampleNames;
	TArray<TArray<float>> SampleData;

	int32 EventsHeadingIndex = INDEX_NONE;

	// Headers
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(Private::SerializeText::Headers);
		TArray<FString> Headings;
		HeaderRow->ParseIntoArray(Headings, TEXT(","), false);
		SampleNames.SetNum(Headings.Num());
		SampleData.SetNum(Headings.Num());
		for (TArray<FString>::TIterator Iter(Headings); Iter; ++Iter)
		{
			const int32 ColumnIndex = Iter.GetIndex();
			if (FStringView(*Iter).Equals(TEXTVIEW("events"), ESearchCase::IgnoreCase))
			{
				EventsHeadingIndex = ColumnIndex;
			}
			else
			{
				SampleNames[ColumnIndex] = MoveTemp(*Iter);
				SampleData[ColumnIndex].SetNum(NumSamples);
			}
		}
	}

	TArray<FString> EventStrings;
	EventStrings.SetNum(NumSamples);

	// Samples
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(Private::SerializeText::Samples);
		ParallelFor(NumSamples, [&](int32 Index)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(Private::SerializeText::Samples::Work);
			// Split the metadata line into segments at commas
			const TCHAR* LinePtr = *Lines[Index];
			FString Token;
			int32 ColumnIndex = 0;
			while (FParse::Token(LinePtr, Token, false, TEXT(',')))
			{
				if (LIKELY(ColumnIndex != EventsHeadingIndex))
				{
					LexFromString(SampleData[ColumnIndex][Index], Token);
				}
				else
				{
					EventStrings[Index] = MoveTemp(Token);
				}

				++ColumnIndex;
			}
		});
	}

	// Events
	TArray<FCsvProfilerEvent> Events;
	if (EventsHeadingIndex != INDEX_NONE)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(Private::SerializeText::Events);
		for (TArray<FString>::TConstIterator Iter(EventStrings); Iter; ++Iter)
		{
			if (Iter->IsEmpty())
			{
				continue;
			}

			ParseTokens(
				*Iter, TEXT(';'),
				[&](FStringView EventToken)
				{ Events.Add({ .Name = FString(EventToken), .Frame = Iter.GetIndex() }); },
				UE::String::EParseTokensOptions::IgnoreCase | UE::String::EParseTokensOptions::SkipEmpty);
		}

		// Cleanup unused stat data for events
		SampleNames.RemoveAt(EventsHeadingIndex, EAllowShrinking::No);
		SampleData.RemoveAt(EventsHeadingIndex, EAllowShrinking::No);
	}

	// Finalize
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(Private::SerializeText::Finalize);
		OutCapture.Samples.Reserve(SampleNames.Num());
		for (int32 SeriesIndex = 0; SeriesIndex < SampleNames.Num(); ++SeriesIndex)
		{
			FCsvProfilerSample& Sample = OutCapture.Samples.Add(SampleNames[SeriesIndex]);
			Sample.Name = SampleNames[SeriesIndex];
			Sample.Values = MoveTemp(SampleData[SeriesIndex]);
		}
		OutCapture.Events = MoveTemp(Events);
		OutCapture.Metadata = MoveTemp(Metadata);
	}
	return true;
}

/**
 * Helper function to decode a 7-bit encoded integer from the parameter archive.
 * @param Ar The archive to read from.
 * @return The decoded integer.
 */
static uint64 Decode7bit(FArchive& Ar)
{
	uint64 Value = 0;
	uint64 ByteIndex = 0;
	bool HasMoreBytes;
	do
	{
		uint8 ByteValue;
		Ar.Serialize(&ByteValue, 1);
		HasMoreBytes = ByteValue & 0x80;
		Value |= uint64(ByteValue & 0x7f) << (ByteIndex * 7);
		++ByteIndex;
	} while (HasMoreBytes);
	return Value;
}

/**
 * Helper function to read a C# BinaryWriter-serialized string into a string builder.
 * @tparam CharType Character type.
 * @param Ar Binary archive to read the string from.
 * @param Builder String builder to write to.
 */
template <typename CharType>
void SerializeCsString(FArchive& Ar, TStringBuilderBase<CharType>& Builder)
{
	// C# BinaryWriter prefixes strings with their length as a 7-bit encoded int
	uint32 StringLength = Decode7bit(Ar);

	if constexpr (TIsCharEncodingCompatibleWith_V<UTF8CHAR, CharType>)
	{
		const int32 Offset = Builder.AddUninitialized(StringLength);
		Ar.Serialize(Builder.GetData() + Offset, StringLength);
	}
	else
	{
		TArray<UTF8CHAR, TInlineAllocator<128>> Buffer;
		Buffer.SetNumUninitialized(StringLength);
		Ar.Serialize(Buffer.GetData(), StringLength);
		const int32 ConvertedLength = FPlatformString::ConvertedLength<CharType>(Buffer.GetData(), StringLength);
		const int32 Offset = Builder.AddUninitialized(ConvertedLength);
		FPlatformString::Convert(Builder.GetData() + Offset, ConvertedLength, Buffer.GetData(), Buffer.Num());
	}
}

/**
 * Helper function to read a C# BinaryWriter-serialized string into a string.
 * @tparam CharType Character type.
 * @param Ar Binary archive to read the string from.
 * @param OutString String to write to.
 */
template <typename CharType>
void SerializeCsString(FArchive& Ar, TString<CharType>& OutString)
{
	// C# BinaryWriter prefixes strings with their length as a 7-bit encoded int
	uint32 StringLength = Decode7bit(Ar);

	TArray<CharType, typename TString<CharType>::AllocatorType>& Data = OutString.GetCharArray();
	if constexpr (TIsCharEncodingCompatibleWith_V<UTF8CHAR, CharType>)
	{
		Data.SetNumUninitialized(StringLength + 1);
		Ar.Serialize(Data.GetData(), StringLength);
		Data[StringLength] = CHARTEXT(CharType, '\0');
	}
	else
	{
		TArray<UTF8CHAR, TInlineAllocator<128>> Buffer;
		Buffer.SetNumUninitialized(StringLength);
		Ar.Serialize(Buffer.GetData(), Buffer.Num());
		const int32 ConvertedLength = FPlatformString::ConvertedLength<CharType>(Buffer.GetData(), Buffer.Num());
		Data.SetNumUninitialized(ConvertedLength + 1);
		FPlatformString::Convert(Data.GetData(), ConvertedLength, Buffer.GetData(), Buffer.Num());
		Data[ConvertedLength] = CHARTEXT(CharType, '\0');
	}
}

/**
 * Helper function to read a C# BinaryWriter-serialized string into a string.
 * @tparam CharType Character type.
 * @param Ar Binary archive to read the string from.
 * @return The string that was read.
 */
template <typename CharType>
TString<CharType> SerializeCsString(FArchive& Ar)
{
	TString<CharType> String;
	SerializeCsString<CharType>(Ar, String);
	return String;
}

/**
 * Helper function to serialize metadata from the parameter archive.
 * @param Ar Binary archive to read the metadata from.
 * @param OutMetadata The container to store the read metadata in.
 */
void SerializeMetadataBin(FArchive& Ar, TMap<FString, FString>& OutMetadata)
{
	int32 NumValues;
	Ar << NumValues;

	OutMetadata.Reserve(NumValues);
	for (int32 Index = 0; Index < NumValues; Index++)
	{
		FString Key = SerializeCsString<TCHAR>(Ar);
		FString Value = SerializeCsString<TCHAR>(Ar);
		OutMetadata.Add(MoveTemp(Key), MoveTemp(Value));
	}
}

/**
 * Serializes binary CSV profiler data. Based on the C# code found in CsvStats.ReadBinFile.
 * @param Ar Binary archive to read the capture data from.
 * @param OutCapture Output container to store the read capture in.
 * @param OutErrors Optional output array to hold any reported errors.
 * @return True if serialization of the capture succeeded.
 */
bool SerializeBin(FArchive& Ar, FCsvProfilerCapture& OutCapture, TArray<FText>* OutErrors = nullptr)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(Private::SerializeBin);

	// Check magic
	{
		TUtf8StringBuilder<256> Builder;
		SerializeCsString(Ar, Builder);
		const FUtf8StringView CsvBinMagic(UTF8TEXTVIEW("CSVBIN"));
		if (Builder != CsvBinMagic)
		{
			if (OutErrors)
			{
				OutErrors->Add(IncorrectFormatText);
			}
			return false;
		}
	}

	ECsvBinVersion Version;
	Ar << Version;
	if (Version < ECsvBinVersion::InitialRelease)
	{
		if (OutErrors)
		{
			OutErrors->Add(UnsupportedVersionText);
		}
		return false;
	}

	// Read flags
	ECsvBinFlags Flags = ECsvBinFlags::None;
	bool bCompressed = false;
	if (Version >= ECsvBinVersion::CompressionSupportAndFlags)
	{
		Ar << Flags;

		ECsvBinCompressionLevel CompressionLevel;
		Ar << CompressionLevel;
		bCompressed = CompressionLevel != ECsvBinCompressionLevel::None;

		if (bCompressed)
		{
			CsvBinCompressionType CompressionType;
			Ar << CompressionType;
			if (CompressionType != CsvBinCompressionType::MsDeflate)
			{
				if (OutErrors)
				{
					OutErrors->Add(UnsupportedCompressionTypeText);
				}
				return false;
			}
		}
		else
		{
			if (OutErrors)
			{
				OutErrors->Add(UncompressedFormatSupportText);
			}
			return false;
		}
	}
	else
	{
		bool bHasMetadata = false;
		Ar << bHasMetadata;
		if (bHasMetadata)
		{
			EnumAddFlags(Flags, ECsvBinFlags::HasMetadata);
		}
	}

	TMap<FString, FString> Metadata;
	if (EnumHasAnyFlags(Flags, ECsvBinFlags::HasMetadata))
	{
		SerializeMetadataBin(Ar, Metadata);
	}

	// Read counts
	int32 EventCount, ValueCount, SampleCount;
	Ar << EventCount << ValueCount << SampleCount;

	// Read sample names
	TMap<FString, FCsvProfilerSample> Samples;
	Samples.Reserve(SampleCount);
	for (int32 Index = 0; Index < SampleCount; Index++)
	{
		Samples.Add(SerializeCsString<TCHAR>(Ar));
	}

	// Read the sample data
	for (int32 Index = 0; Index < SampleCount; Index++)
	{
		FString SampleName = SerializeCsString<TCHAR>(Ar);

		FCsvProfilerSample* FoundSample = Samples.Find(SampleName);
		if (!FoundSample)
		{
			if (OutErrors)
			{
				OutErrors->Add(
					FText::FormatNamed(SampleDataNotFoundText, FString(SampleNameKey), FText::FromString(SampleName)));
			}
			return false;
		}

		FCsvProfilerSample& Sample = *FoundSample;

		Sample.Name = MoveTemp(SampleName);
		Ar << Sample.Average;
		Ar << Sample.Total;

		int32 StatSizeBytes;
		Ar << StatSizeBytes;

		if (bCompressed)
		{
			int32 CompressedBufferLength;
			Ar << CompressedBufferLength;

			TArray<uint8> CompressedBuffer;
			CompressedBuffer.SetNumUninitialized(CompressedBufferLength);
			Ar.Serialize(CompressedBuffer.GetData(), CompressedBufferLength);

			int32 UncompressedBufferLength = sizeof(float) * ValueCount;
			TArray<float> UncompressedValuesBuffer;
			UncompressedValuesBuffer.SetNumUninitialized(ValueCount);

			// FCompression::UncompressMemory uses the CompressionData parameter as the window size zlib uses.
			// Per zlib's documentation, a window size value between -8 and -15 will do a raw inflate, which
			// is what we want as the compressed data doesn't have any headers.
			constexpr int32 WindowSizeRawDeflate = -15;
			if (!FCompression::UncompressMemory(
					/*          FormatName = */ NAME_Zlib,
					/*  UncompressedBuffer = */ UncompressedValuesBuffer.GetData(),
					/*    UncompressedSize = */ UncompressedBufferLength,
					/*    CompressedBuffer = */ CompressedBuffer.GetData(),
					/*      CompressedSize = */ CompressedBufferLength,
					/*               Flags = */ COMPRESS_NoFlags,
					/*     CompressionData = */ WindowSizeRawDeflate))
			{
				return false;
			}

			Sample.Values = MoveTemp(UncompressedValuesBuffer);
		}
		else
		{
			unimplemented();
		}
	}

	// Read the event data
	TArray<FCsvProfilerEvent> Events;
	Events.Reserve(SampleCount);
	for (int i = 0; i < EventCount; i++)
	{
		int32 Frame;
		Ar << Frame;
		FString Name = SerializeCsString<TCHAR>(Ar);
		Events.Add({.Name = MoveTemp(Name), .Frame = Frame});
	}

	// Success - move into the output capture container.
	OutCapture.Samples = MoveTemp(Samples);
	OutCapture.Events = MoveTemp(Events);
	OutCapture.Metadata = MoveTemp(Metadata);
	return true;
}

}  // namespace CsvUtil::Private

bool ReadFromCsv(FCsvProfilerCapture& OutCapture, const TCHAR* FilePath, TArray<FText>* OutErrors /*= nullptr*/)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ReadFromCsv);
	FScopedSlowTask SlowTask(1, LOCTEXT("ReadFromCsv", "Reading CSV data"));
	SlowTask.MakeDialog();
	SlowTask.EnterProgressFrame();
	OutCapture = FCsvProfilerCapture{};
	TArray<FString> Lines;
	if (FFileHelper::LoadFileToStringArray(Lines, FilePath))
	{
		return CsvUtils::Private::SerializeText(Lines, OutCapture, OutErrors);
	}
	return false;
}

bool ReadFromCsvBin(FCsvProfilerCapture& OutCapture, const TCHAR* FilePath, TArray<FText>* OutErrors /*= nullptr*/)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ReadFromCsvBin);
	FScopedSlowTask SlowTask(1, LOCTEXT("ReadFromCsvBin", "Reading CSV data"));
	SlowTask.MakeDialog();
	SlowTask.EnterProgressFrame();
	OutCapture = FCsvProfilerCapture{};
	if (TUniquePtr<FArchive> Archive(IFileManager::Get().CreateFileReader(FilePath)); Archive.IsValid())
	{
		return CsvUtils::Private::SerializeBin(*Archive, OutCapture, OutErrors);
	}
	return false;
}

}

#undef LOCTEXT_NAMESPACE