// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "Containers/UnrealString.h"

namespace CsvUtils
{

	struct FCsvProfilerSample
	{
		FString Name;
		TArray<float> Values;
		float Average = 0.0f;
		double Total = 0.0;
	};

	struct FCsvProfilerEvent
	{
		FString Name;
		int32 Frame = INDEX_NONE;
	};

	struct FCsvProfilerCapture
	{
		TMap<FString, FCsvProfilerSample> Samples;
		TArray<FCsvProfilerEvent> Events;
		TMap<FString, FString> Metadata;
	};

	/**
	 * Reads a csv profiler capture from the parameter file path into the parameter container.
	 * @param OutCapture The destination capture container.
	 * @param FilePath The source file path.
	 * @param OutErrors Optional output array to hold any reported errors.
	 * @return True if the capture was successfully read.
	 */
	CSVUTILS_API bool ReadFromCsv(FCsvProfilerCapture& OutCapture, const TCHAR* FilePath, TArray<FText>* OutErrors = nullptr);

	/**
	 * Reads a binary csv profiler capture from the parameter file path into the parameter container.
	 * @param OutCapture The destination capture container.
	 * @param FilePath The source file path.
	 * @param OutErrors Optional output array to hold any reported errors.
	 * @return True if the capture was successfully read.
	 */
	CSVUTILS_API bool ReadFromCsvBin(FCsvProfilerCapture& OutCapture, const TCHAR* FilePath, TArray<FText>* OutErrors = nullptr);

}